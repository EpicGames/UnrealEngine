// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_PropertyAccess.h"
#include "AnimGraphNode_Base.h"
#include "FindInBlueprintManager.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "EditorCategoryUtils.h"
#include "BlueprintNodeSpawner.h"
#include "K2Node_VariableGet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "AnimBlueprintExtension_PropertyAccess.h"
#include "IPropertyAccessEditor.h"
#include "Features/IModularFeatures.h"
#include "IAnimBlueprintCompilationContext.h"
#include "IPropertyAccessBlueprintBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_PropertyAccess)

#define LOCTEXT_NAMESPACE "K2Node_PropertyAccess"

void UK2Node_PropertyAccess::CreateClassVariablesFromBlueprint(IAnimBlueprintVariableCreationContext& InCreationContext)
{
	GeneratedPropertyName = NAME_None;

	const bool bRequiresCachedVariable = !bWasResolvedThreadSafe || UAnimBlueprintExtension_PropertyAccess::ContextRequiresCachedVariable(ContextId);
	
	if(ResolvedPinType != FEdGraphPinType() && ResolvedPinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard && bRequiresCachedVariable)
	{
		// Create internal generated destination property (only if we were not identified as thread safe)
		if(FProperty* DestProperty = InCreationContext.CreateUniqueVariable(this, ResolvedPinType))
		{
			GeneratedPropertyName = DestProperty->GetFName();
			DestProperty->SetMetaData(TEXT("BlueprintCompilerGeneratedDefaults"), TEXT("true"));
		}
	}
}

void UK2Node_PropertyAccess::ExpandNode(FKismetCompilerContext& InCompilerContext, UEdGraph* InSourceGraph)
{
	Super::ExpandNode(InCompilerContext, InSourceGraph);
	
	ResolvePropertyAccess();

	UK2Node_PropertyAccess* OriginalNode = CastChecked<UK2Node_PropertyAccess>(InCompilerContext.MessageLog.FindSourceObject(this));
	OriginalNode->CompiledContext = FText();
	OriginalNode->CompiledContextDesc = FText();

	TUniquePtr<IAnimBlueprintCompilationContext> CompilationContext = IAnimBlueprintCompilationContext::Get(InCompilerContext);
	UAnimBlueprint* AnimBlueprint = CastChecked<UAnimBlueprint>(GetBlueprint());
	UAnimBlueprintExtension_PropertyAccess* PropertyAccessExtension = UAnimBlueprintExtension::FindExtension<UAnimBlueprintExtension_PropertyAccess>(AnimBlueprint);

	if (!PropertyAccessExtension)
	{
		// Child anim blueprints cannot modify or augment the property access library of their parent classes, so any additional 
		// property access nodes in child classes will not work at present. To address this (and likely a number of other missing 
		// child anim BP features) we would need to allow child anim BPs to perform a 'full' compilation, so they can add additional
		// anim graph nodes & property access libraries, amongst other things.
		if (UAnimBlueprint* ParentAnimBP = UAnimBlueprint::GetParentAnimBlueprint(AnimBlueprint))
		{
			InCompilerContext.MessageLog.Error(*LOCTEXT("PropertyAccess_AnimBlueprintExtension_PropertyAccess_NotFound_ChildAnimBP", "@@ is currently unsupported in Child Animation Blueprints.").ToString(), this);
		}
		else
		{
			InCompilerContext.MessageLog.Error(*LOCTEXT("PropertyAccess_AnimBlueprintExtension_PropertyAccess_NotFound", "Internal compiler error: Property access extension not found for @@.").ToString(), this);
		}
		return;
	}
	
	if(GeneratedPropertyName != NAME_None)
	{
		TArray<FString> DestPropertyPath;

		// We are using an intermediate object-level property (as we need to call this access and cache its result
		// until later) 
		DestPropertyPath.Add(GeneratedPropertyName.ToString());

		// Create a copy event in the complied generated class
		FPropertyAccessHandle Handle = PropertyAccessExtension->AddCopy(Path, DestPropertyPath, ContextId, this);
		
		PostLibraryCompiledHandle = PropertyAccessExtension->OnPostLibraryCompiled().AddLambda([this, OriginalNode, PropertyAccessExtension, Handle](IAnimBlueprintCompilationBracketContext& /*InCompilationContext*/, IAnimBlueprintGeneratedClassCompiledData& /*OutCompiledData*/)
		{
			const FCompiledPropertyAccessHandle CompiledHandle = PropertyAccessExtension->GetCompiledHandle(Handle);
			if(CompiledHandle.IsValid())
			{
				OriginalNode->CompiledContext = UAnimBlueprintExtension_PropertyAccess::GetCompiledHandleContext(CompiledHandle);
				OriginalNode->CompiledContextDesc = UAnimBlueprintExtension_PropertyAccess::GetCompiledHandleContextDesc(CompiledHandle);
			}
			PropertyAccessExtension->OnPostLibraryCompiled().Remove(PostLibraryCompiledHandle);
		});
		
		// Replace us with a get node
		UK2Node_VariableGet* VariableGetNode = InCompilerContext.SpawnIntermediateNode<UK2Node_VariableGet>(this, InSourceGraph);
		VariableGetNode->VariableReference.SetSelfMember(GeneratedPropertyName);
		VariableGetNode->AllocateDefaultPins();

		// Move pin links from Get node we are expanding, to the new pure one we've created
		UEdGraphPin* VariableValuePin = VariableGetNode->GetValuePin();
		check(VariableValuePin);
		InCompilerContext.MovePinLinksToIntermediate(*GetOutputPin(), *VariableValuePin);
	}
	else
	{
		const bool bRequiresCachedVariable = !bWasResolvedThreadSafe || UAnimBlueprintExtension_PropertyAccess::ContextRequiresCachedVariable(ContextId);
		check(!bRequiresCachedVariable);

		UEnum* EnumClass = StaticEnum<EAnimPropertyAccessCallSite>();
		check(EnumClass != nullptr);

		OriginalNode->CompiledContext = EnumClass->GetDisplayNameTextByValue((int32)EAnimPropertyAccessCallSite::WorkerThread_Unbatched);
		OriginalNode->CompiledContextDesc = EnumClass->GetToolTipTextByIndex((int32)EAnimPropertyAccessCallSite::WorkerThread_Unbatched);
		
		PropertyAccessExtension->ExpandPropertyAccess(InCompilerContext, Path, InSourceGraph, GetOutputPin());
	}
}

void UK2Node_PropertyAccess::ResolvePropertyAccess() const
{
	if(UBlueprint* Blueprint = GetBlueprint())
	{
		ResolvedProperty = nullptr;
		ResolvedArrayIndex = INDEX_NONE;
		FProperty* PropertyToResolve = nullptr;
		if(IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
		{
			IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
			FPropertyAccessResolveResult Result = PropertyAccessEditor.ResolvePropertyAccess(Blueprint->SkeletonGeneratedClass, Path, PropertyToResolve, ResolvedArrayIndex);
			bWasResolvedThreadSafe = Result.bIsThreadSafe;
			ResolvedProperty = PropertyToResolve;
		}
	}
	else
	{
		ResolvedProperty = nullptr;
		ResolvedArrayIndex = INDEX_NONE;
	}
}

void UK2Node_PropertyAccess::SetPath(const TArray<FString>& InPath)
{
	Path = InPath;

	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
	TextPath = PropertyAccessEditor.MakeTextPath(Path, GetBlueprint()->SkeletonGeneratedClass);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	ResolvePropertyAccess();
	ReconstructNode();
}

void UK2Node_PropertyAccess::SetPath(TArray<FString>&& InPath)
{
	Path = MoveTemp(InPath);
	
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
	TextPath = PropertyAccessEditor.MakeTextPath(Path, GetBlueprint()->SkeletonGeneratedClass);
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	ResolvePropertyAccess();
	ReconstructNode();
}

void UK2Node_PropertyAccess::ClearPath()
{
	Path.Empty();
	TextPath = FText();
	FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	ResolvedProperty = nullptr;
	ResolvedArrayIndex = INDEX_NONE;
	ReconstructNode();
}

void UK2Node_PropertyAccess::AllocatePins(UEdGraphPin* InOldOutputPin)
{
	// Resolve leaf to try to get a valid property type for an output pin
	ResolvePropertyAccess();

	if(UBlueprint* Blueprint = GetBlueprint())
	{
		// Keep text path up to date
		IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
		TextPath = PropertyAccessEditor.MakeTextPath(Path, GetBlueprint()->SkeletonGeneratedClass);
		
		UEdGraphPin* OutputPin = nullptr;
	
		if(InOldOutputPin != nullptr && InOldOutputPin->LinkedTo.Num() > 0 && InOldOutputPin->PinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard)
		{
			// Use old output pin if we have one and it is connected
			OutputPin = CreatePin(EGPD_Output, InOldOutputPin->PinType, TEXT("Value"));
			ResolvedPinType = InOldOutputPin->PinType;
		}

		if(OutputPin == nullptr && ResolvedProperty.Get() != nullptr)
		{
			// Otherwise use the resolved property
			FProperty* PropertyToUse = ResolvedProperty.Get();
			if(FArrayProperty* ArrayProperty = CastField<FArrayProperty>(PropertyToUse))
			{
				if(ResolvedArrayIndex != INDEX_NONE)
				{
					PropertyToUse = ArrayProperty->Inner;
				}
			}

			// Try to create a pin for the property
			const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();
			if(K2Schema->ConvertPropertyToPinType(PropertyToUse, ResolvedPinType))
			{
				OutputPin = CreatePin(EGPD_Output, ResolvedPinType, TEXT("Value"));
			}
		}

		if(OutputPin == nullptr)
		{
			// Cant resolve a type from the path, make a wildcard pin to begin with
			OutputPin = CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Wildcard, TEXT("Value"));
			ResolvedPinType = OutputPin->PinType;
		}
	}
}

void UK2Node_PropertyAccess::AllocateDefaultPins()
{
	AllocatePins();
}

void UK2Node_PropertyAccess::ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins)
{
	// First find the old output pin, if any
	UEdGraphPin* OldOutputPin = nullptr;
	if(UEdGraphPin** OldOutputPinPtr = OldPins.FindByPredicate([](UEdGraphPin* InPin){ return InPin->PinName == TEXT("Value"); }))
	{
		OldOutputPin = *OldOutputPinPtr;
	}

	AllocatePins(OldOutputPin);

	RestoreSplitPins(OldPins);
}

FText UK2Node_PropertyAccess::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("PropertyAccess", "Property Access");
}

void UK2Node_PropertyAccess::AddPinSearchMetaDataInfo(const UEdGraphPin* InPin, TArray<struct FSearchTagDataPair>& OutTaggedMetaData) const
{
	Super::AddPinSearchMetaDataInfo(InPin, OutTaggedMetaData);

	// Only one pin on the node so no need to check it here
	if(!TextPath.IsEmpty())
	{
		OutTaggedMetaData.Emplace(FText::FromString(TEXT("Binding")), TextPath);
	}
}

void UK2Node_PropertyAccess::PinConnectionListChanged(UEdGraphPin* Pin)
{
	if(Pin && Pin->PinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard && Pin->LinkedTo.Num() > 0)
	{
		Pin->PinType = ResolvedPinType = Pin->LinkedTo[0]->PinType;
	}
}

void UK2Node_PropertyAccess::GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	UClass* NodeClass = GetClass();
	if (ActionRegistrar.IsOpenForRegistration(NodeClass))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(NodeClass);
		check(NodeSpawner);
		ActionRegistrar.AddBlueprintAction(NodeClass, NodeSpawner);
	}
}

FText UK2Node_PropertyAccess::GetMenuCategory() const
{
	return FEditorCategoryUtils::GetCommonCategory(FCommonEditorCategory::Variables);
}

bool UK2Node_PropertyAccess::IsCompatibleWithGraph(UEdGraph const* TargetGraph) const
{
	IPropertyAccessBlueprintBinding::FContext BindingContext;
	BindingContext.Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	BindingContext.Graph = TargetGraph;
	BindingContext.Node = this;
	BindingContext.Pin = FindPin(TEXT("Value"));
	
	// Check any property access blueprint bindings we might have registered
	for(IPropertyAccessBlueprintBinding* Binding : IModularFeatures::Get().GetModularFeatureImplementations<IPropertyAccessBlueprintBinding>("PropertyAccessBlueprintBinding"))
	{
		if(Binding->CanBindToContext(BindingContext))
		{
			return true;
		}
	}

	return false;
}

FText UK2Node_PropertyAccess::GetTooltipText() const
{
	return LOCTEXT("PropertyAccessTooltip", "Accesses properties according to property path");
}

bool UK2Node_PropertyAccess::HasResolvedPinType() const
{
	return ResolvedPinType != FEdGraphPinType() && ResolvedPinType.PinCategory != UEdGraphSchema_K2::PC_Wildcard;
}

void UK2Node_PropertyAccess::PostEditUndo()
{
	Super::PostEditUndo();
	
	ResolvePropertyAccess();
}

void UK2Node_PropertyAccess::HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName)
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	UClass* SkeletonVariableClass = FBlueprintEditorUtils::GetMostUpToDateClass(InVariableClass);

	// See if the path references the variable
	TArray<int32> RenameIndices;
	IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
	ResolveArgs.bUseMostUpToDateClasses = true;
	ResolveArgs.PropertyFunction = [InOldVarName, SkeletonVariableClass, &RenameIndices](int32 InSegmentIndex, FProperty* InProperty, int32 InStaticArrayIndex)
	{
		UClass* OwnerClass = InProperty->GetOwnerClass();
		if(OwnerClass && InProperty->GetFName() == InOldVarName && OwnerClass->IsChildOf(SkeletonVariableClass))
		{
			RenameIndices.Add(InSegmentIndex);
		}
	};
	
	PropertyAccessEditor.ResolvePropertyAccess(GetBlueprint()->SkeletonGeneratedClass, Path, ResolveArgs);

	// Rename any references we found
	for(const int32& RenameIndex : RenameIndices)
	{
		Path[RenameIndex] = InNewVarName.ToString();
		TextPath = PropertyAccessEditor.MakeTextPath(Path, GetBlueprint()->SkeletonGeneratedClass);
	}
}

void UK2Node_PropertyAccess::HandleFunctionRenamed(UBlueprint* InBlueprint, UClass* InFunctionClass, UEdGraph* InGraph, const FName& InOldFuncName, const FName& InNewFuncName)
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	UClass* SkeletonFunctionClass = FBlueprintEditorUtils::GetMostUpToDateClass(InFunctionClass);

	// See if the path references the variable
	TArray<int32> RenameIndices;
	IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
	ResolveArgs.bUseMostUpToDateClasses = true;
	ResolveArgs.FunctionFunction = [InOldFuncName, SkeletonFunctionClass, &RenameIndices](int32 InSegmentIndex, UFunction* InFunction, FProperty* InReturnProperty)
	{
		const UClass* OwnerSkeletonFunctionClass = FBlueprintEditorUtils::GetMostUpToDateClass(InFunction->GetOuterUClass());
		if(OwnerSkeletonFunctionClass && InFunction->GetFName() == InOldFuncName && OwnerSkeletonFunctionClass->IsChildOf(SkeletonFunctionClass))
		{
			RenameIndices.Add(InSegmentIndex);
		}
	};
	
	PropertyAccessEditor.ResolvePropertyAccess(GetBlueprint()->SkeletonGeneratedClass, Path, ResolveArgs);

	// Rename any references we found
	for(const int32& RenameIndex : RenameIndices)
	{
		Path[RenameIndex] = InNewFuncName.ToString();
		TextPath = PropertyAccessEditor.MakeTextPath(Path, GetBlueprint()->SkeletonGeneratedClass);
	}
}

void UK2Node_PropertyAccess::ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement)
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
	
	UClass* SkeletonClass = InBlueprint->SkeletonGeneratedClass;

	FMemberReference Source = InSource;
	FProperty* SourceProperty = Source.ResolveMember<FProperty>(InBlueprint);
	FMemberReference Replacement = InReplacement;
	FProperty* ReplacementProperty = Replacement.ResolveMember<FProperty>(InReplacementBlueprint);

	// See if the path references the variable
	TArray<int32> ReplaceIndices;
	IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
	ResolveArgs.bUseMostUpToDateClasses = true;
	ResolveArgs.PropertyFunction = [SourceProperty, &ReplaceIndices](int32 InSegmentIndex, FProperty* InProperty, int32 InStaticArrayIndex)
	{
		if(InProperty == SourceProperty)
		{
			ReplaceIndices.Add(InSegmentIndex);
		}
	};
	
	PropertyAccessEditor.ResolvePropertyAccess(GetBlueprint()->SkeletonGeneratedClass, Path, ResolveArgs);

	// Replace any references we found
	for(const int32& RenameIndex : ReplaceIndices)
	{
		Path[RenameIndex] = ReplacementProperty->GetName();
		TextPath = PropertyAccessEditor.MakeTextPath(Path, GetBlueprint()->SkeletonGeneratedClass);
	}
}

bool UK2Node_PropertyAccess::ReferencesVariable(const FName& InVarName, const UStruct* InScope) const
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	const UClass* SkeletonVariableClass = FBlueprintEditorUtils::GetMostUpToDateClass(Cast<UClass>(InScope));

	// See if the path references the variable
	bool bReferencesVariable = false;
	
	IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
	ResolveArgs.bUseMostUpToDateClasses = true;
	ResolveArgs.PropertyFunction = [InVarName, SkeletonVariableClass, &bReferencesVariable](int32 InSegmentIndex, FProperty* InProperty, int32 InStaticArrayIndex)
	{
		if(SkeletonVariableClass)
		{
			const UClass* OwnerSkeletonVariableClass = FBlueprintEditorUtils::GetMostUpToDateClass(Cast<UClass>(InProperty->GetOwnerStruct()));

			if(OwnerSkeletonVariableClass && InProperty->GetFName() == InVarName && OwnerSkeletonVariableClass->IsChildOf(SkeletonVariableClass))
			{
				bReferencesVariable = true;
			}
		}
		else if(InProperty->GetFName() == InVarName)
		{
			bReferencesVariable = true;
		}
	};
	
	PropertyAccessEditor.ResolvePropertyAccess(GetBlueprint()->SkeletonGeneratedClass, Path, ResolveArgs);
	
	return bReferencesVariable;
}

bool UK2Node_PropertyAccess::ReferencesFunction(const FName& InFunctionName, const UStruct* InScope) const
{
	IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

	const UClass* SkeletonFunctionClass = FBlueprintEditorUtils::GetMostUpToDateClass(Cast<UClass>(InScope));

	// See if the path references the function
	bool bReferencesFunction = false;

	IPropertyAccessEditor::FResolvePropertyAccessArgs ResolveArgs;
	ResolveArgs.bUseMostUpToDateClasses = true;
	ResolveArgs.FunctionFunction = [InFunctionName, SkeletonFunctionClass, &bReferencesFunction](int32 InSegmentIndex, UFunction* InFunction, FProperty* InProperty)
	{
		if (SkeletonFunctionClass)
		{
			const UClass* OwnerSkeletonFunctionClass = FBlueprintEditorUtils::GetMostUpToDateClass(InFunction->GetOuterUClass());

			if (OwnerSkeletonFunctionClass && InFunction->GetFName() == InFunctionName && OwnerSkeletonFunctionClass->IsChildOf(SkeletonFunctionClass))
			{
				bReferencesFunction = true;
			}
		}
		else if (InFunction->GetFName() == InFunctionName)
		{
			bReferencesFunction = true;
		}
	};

	PropertyAccessEditor.ResolvePropertyAccess(GetBlueprint()->SkeletonGeneratedClass, Path, ResolveArgs);

	return bReferencesFunction;
}

const FProperty* UK2Node_PropertyAccess::GetResolvedProperty() const
{
	if(const FProperty* Property = ResolvedProperty.Get())
	{
		return Property;
	}
	
	ResolvePropertyAccess();

	return ResolvedProperty.Get();
}

#undef LOCTEXT_NAMESPACE

