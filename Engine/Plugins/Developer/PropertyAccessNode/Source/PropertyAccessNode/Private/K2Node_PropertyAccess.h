// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "K2Node.h"
#include "IClassVariableCreator.h"
#include "EdGraph/EdGraphPin.h"

#include "UObject/FieldPath.h"
#include "K2Node_PropertyAccess.generated.h"

struct FSearchTagDataPair;

UCLASS(MinimalAPI)
class UK2Node_PropertyAccess : public UK2Node, public IClassVariableCreator
{
public:
	GENERATED_BODY()

	/** Set the path and attempt to resolve the leaf property. */
	void SetPath(const TArray<FString>& InPath);
	void SetPath(TArray<FString>&& InPath);

	/** Clear the path */
	void ClearPath();

	/** Get the path */
	const TArray<FString>& GetPath() const { return Path; }

	/** Get the path as text */
	const FText& GetTextPath() const { return TextPath; }

	/** Get the resolved leaf property, if any. */
	const FProperty* GetResolvedProperty() const;

	/** Get the resolved leaf property's array index, if any. */
	int32 GetResolvedArrayIndex() const { return ResolvedArrayIndex; }

	/** Get the resolved pin type, if any. */
	const FEdGraphPinType& GetResolvedPinType() const { return ResolvedPinType; }

	/** Check whether we have a resolved pin type */
	bool HasResolvedPinType() const;
	
	/** Get the output pin */
	UEdGraphPin* GetOutputPin() const { return FindPinChecked(TEXT("Value"), EGPD_Output); }

	/** Get the context Id */
	FName GetContextId() const { return ContextId; }

	/** Set the context Id */
	void SetContextId(const FName& InContextId) { ContextId = InContextId; }

	/** Get the the context that this access was assigned during compilation. This can be empty. */
	const FText& GetCompiledContext() const { return CompiledContext; }
	
	/** Get the description of the context that this access was assigned during compilation. This can be empty. */
	const FText& GetCompiledContextDesc() const { return CompiledContextDesc; }

private:
	/** Path that this access exposes */
	UPROPERTY()
	TArray<FString> Path;

	/** Path as text, for display */
	UPROPERTY()
	FText TextPath;

	/** Resolved pin type */
	UPROPERTY()
	FEdGraphPinType ResolvedPinType;

	/** Generated property created during compilation */
	UPROPERTY()
	FName GeneratedPropertyName = NAME_None;

	/** Property access context (set by the user) that is intended to be used */
	UPROPERTY()
	FName ContextId;
	
	/** Resolved leaf property for the path, NULL if path cant be resolved or is empty. */
	mutable TFieldPath<FProperty> ResolvedProperty;

	/** Resolved array index, if the leaf property is an array. If this is INDEX_NONE then the property refers to the entire array */
	mutable int32 ResolvedArrayIndex = INDEX_NONE;

	/** Whether the path was resolved as thread safe the last time the ResolvedProperty was updated */
	mutable bool bWasResolvedThreadSafe = false;

	/** Handle used to hook into property access compilation machinery */
	FDelegateHandle PostLibraryCompiledHandle;

	/** The context that this access was assigned during compilation */
	FText CompiledContext;

	/** Description of the context that this access was assigned during compilation (used for tooltips) */
	FText CompiledContextDesc;
	
	// IClassVariableCreator interface
	virtual void CreateClassVariablesFromBlueprint(IAnimBlueprintVariableCreationContext& InCreationContext) override;

	// UObject interface
	virtual void PostEditUndo() override;
	
	// UEdGraphNode interface
	virtual void AllocateDefaultPins() override;
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual void AddPinSearchMetaDataInfo(const UEdGraphPin* InPin, TArray<FSearchTagDataPair>& OutTaggedMetaData) const override;
	virtual void PinConnectionListChanged(UEdGraphPin* Pin) override;
	virtual bool IsCompatibleWithGraph(UEdGraph const* TargetGraph) const override;
	
	// UK2Node interface
	virtual void ExpandNode(class FKismetCompilerContext& CompilerContext, UEdGraph* SourceGraph) override;
	virtual void ReallocatePinsDuringReconstruction(TArray<UEdGraphPin*>& OldPins) override;
	virtual bool IsNodePure() const override { return true; }
	virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual FText GetMenuCategory() const override;
	virtual bool DrawNodeAsVariable() const override { return true; }
	virtual FText GetTooltipText() const override;
	virtual void HandleVariableRenamed(UBlueprint* InBlueprint, UClass* InVariableClass, UEdGraph* InGraph, const FName& InOldVarName, const FName& InNewVarName) override;
	virtual void HandleFunctionRenamed(UBlueprint* InBlueprint, UClass* InFunctionClass, UEdGraph* InGraph, const FName& InOldFuncName, const FName& InNewFuncName) override;
	virtual void ReplaceReferences(UBlueprint* InBlueprint, UBlueprint* InReplacementBlueprint, const FMemberReference& InSource, const FMemberReference& InReplacement) override;
	virtual bool ReferencesVariable(const FName& InVarName, const UStruct* InScope) const override;
	virtual bool ReferencesFunction(const FName& InFunctionName,const UStruct* InScope) const override;
	
	/** Helper function for pin allocation */
	void AllocatePins(UEdGraphPin* InOldOutputPin = nullptr);

	/** Attempt to resolve the path to a leaf property. Const function that modifies mutable internally cached values */
	void ResolvePropertyAccess() const;
};
