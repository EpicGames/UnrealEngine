// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAccessNodeFactory.h"
#include "SPropertyAccessNode.h"
#include "K2Node_PropertyAccess.h"

TSharedPtr<SGraphNode> FPropertyAccessNodeFactory::CreateNode(UEdGraphNode* InNode) const
{
	if(UK2Node_PropertyAccess* K2Node_PropertyAccess = Cast<UK2Node_PropertyAccess>(InNode))
	{
		return SNew(SPropertyAccessNode, K2Node_PropertyAccess);
	}

	return nullptr;
}