// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"

class FPropertyAccessNodeFactory : public FGraphPanelNodeFactory
{
	// FGraphPanelNodeFactory interface
	virtual TSharedPtr<SGraphNode> CreateNode(UEdGraphNode* InNode) const override;
};
