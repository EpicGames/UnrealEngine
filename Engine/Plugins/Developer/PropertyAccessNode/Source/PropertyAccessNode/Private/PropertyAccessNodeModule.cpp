// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "PropertyAccessNodeFactory.h"

class FPropertyAccessNodeModule : public IModuleInterface
{
public:
	// IModuleInterface interface
	virtual void StartupModule()
	{
		PropertyAccessNodeFactory = MakeShared<FPropertyAccessNodeFactory>();
		FEdGraphUtilities::RegisterVisualNodeFactory(PropertyAccessNodeFactory);
	}

	virtual void ShutdownModule()
	{
		FEdGraphUtilities::UnregisterVisualNodeFactory(PropertyAccessNodeFactory);
	}

	TSharedPtr<FPropertyAccessNodeFactory> PropertyAccessNodeFactory;
};

IMPLEMENT_MODULE(FPropertyAccessNodeModule, PropertyAccessNode)
