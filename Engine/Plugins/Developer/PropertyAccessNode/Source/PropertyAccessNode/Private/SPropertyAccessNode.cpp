// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyAccessNode.h"
#include "Rendering/SlateRenderer.h"
#include "SLevelOfDetailBranchNode.h"
#include "IPropertyAccessEditor.h"
#include "Widgets/Layout/SSpacer.h"
#include "K2Node_PropertyAccess.h"
#include "SGraphPin.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Features/IModularFeatures.h"
#include "Widgets/SBoxPanel.h"
#include "Features/IModularFeatures.h"
#include "IPropertyAccessBlueprintBinding.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SPropertyAccessNode"

void SPropertyAccessNode::Construct(const FArguments& InArgs, UK2Node_PropertyAccess* InNode)
{
	GraphNode = InNode;

	SetCursor( EMouseCursor::CardinalCross );
	
	UpdateGraphNode();

	// Centre the pin slot
	if(RightNodeBox->GetChildren()->Num() > 0)
	{
		SVerticalBox::FSlot& PinSlot = RightNodeBox->GetSlot(0);
		PinSlot.SetVerticalAlignment(VAlign_Center);
		PinSlot.SetFillHeight(1.0f);
	}
}

bool SPropertyAccessNode::CanBindProperty(FProperty* InProperty) const
{
	UK2Node_PropertyAccess* K2Node_PropertyAccess = CastChecked<UK2Node_PropertyAccess>(GraphNode);

	if(InProperty == nullptr)
	{
		return true;
	}

	if(IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
	{
		IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

		FEdGraphPinType ResolvedPinType = K2Node_PropertyAccess->GetResolvedPinType();
		FEdGraphPinType PropertyPinType;
		if(ResolvedPinType.PinCategory == UEdGraphSchema_K2::PC_Wildcard)
		{
			// If a property is not already resolved, we allow any type...
			return true;
		}
		else if(Schema->ConvertPropertyToPinType(InProperty, PropertyPinType))
		{
			// ...unless we have a valid pin type already
			return PropertyAccessEditor.GetPinTypeCompatibility(PropertyPinType, ResolvedPinType) != EPropertyAccessCompatibility::Incompatible;
		}
		else
		{
			// Otherwise fall back on a resolved property
			const FProperty* ResolvedProperty = K2Node_PropertyAccess->GetResolvedProperty();
			if(ResolvedProperty != nullptr)
			{
				const FProperty* PropertyToUse = ResolvedProperty;
				if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ResolvedProperty))
				{
					if(K2Node_PropertyAccess->GetResolvedArrayIndex() != INDEX_NONE)
					{
						PropertyToUse = ArrayProperty->Inner;
					}
				}

				// Note: We support type promotion here
				return PropertyAccessEditor.GetPropertyCompatibility(InProperty, PropertyToUse) != EPropertyAccessCompatibility::Incompatible;
			}
		}
	}

	return false;
}

TSharedRef<SWidget> SPropertyAccessNode::UpdateTitleWidget(FText InTitleText, TSharedPtr<SWidget> InTitleWidget, EHorizontalAlignment& InOutTitleHAlign, FMargin& InOutTitleMargin) const
{
	UK2Node_PropertyAccess* K2Node_PropertyAccess = CastChecked<UK2Node_PropertyAccess>(GraphNode);

	FPropertyBindingWidgetArgs Args;

	Args.OnCanBindPropertyWithBindingChain = FOnCanBindPropertyWithBindingChain::CreateLambda([this](FProperty* InProperty, TConstArrayView<FBindingChainElement> InBindingChain)
	{
		return CanBindProperty(InProperty);
	});

	Args.OnCanBindFunction = FOnCanBindFunction::CreateLambda([this](UFunction* InFunction)
	{
		if(InFunction->NumParms != 1 || InFunction->GetReturnProperty() == nullptr || !InFunction->HasAnyFunctionFlags(FUNC_BlueprintPure))
		{
			return false;
		}

		// check the return property directly
		return CanBindProperty(InFunction->GetReturnProperty());
	});

	Args.OnCanBindToClass = FOnCanBindToClass::CreateLambda([](UClass* InClass)
	{
		return true;
	});

	Args.OnAddBinding = FOnAddBinding::CreateLambda([K2Node_PropertyAccess](FName InPropertyName, const TArray<FBindingChainElement>& InBindingChain)
	{
		if(IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
		{
			IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");

			K2Node_PropertyAccess->Modify();
			
			TArray<FString> StringPath;
			PropertyAccessEditor.MakeStringPath(InBindingChain, StringPath);
			K2Node_PropertyAccess->SetPath(MoveTemp(StringPath));

			FBlueprintEditorUtils::MarkBlueprintAsModified(K2Node_PropertyAccess->GetBlueprint());
		}
	});

	Args.OnRemoveBinding = FOnRemoveBinding::CreateLambda([K2Node_PropertyAccess](FName InPropertyName)
	{
		K2Node_PropertyAccess->Modify();
		
		K2Node_PropertyAccess->ClearPath();
		FBlueprintEditorUtils::MarkBlueprintAsModified(K2Node_PropertyAccess->GetBlueprint());
	});

	Args.OnCanRemoveBinding = FOnCanRemoveBinding::CreateLambda([K2Node_PropertyAccess](FName InPropertyName)
	{
		return K2Node_PropertyAccess->GetPath().Num() > 0;
	});

	Args.CurrentBindingText = MakeAttributeLambda([K2Node_PropertyAccess]()
	{
		const FText& TextPath = K2Node_PropertyAccess->GetTextPath();
		return TextPath.IsEmpty() ? LOCTEXT("Bind", "Bind") : TextPath;
	});

	Args.CurrentBindingToolTipText = MakeAttributeLambda([K2Node_PropertyAccess]()
	{
		const FText& TextPath = K2Node_PropertyAccess->GetTextPath();
		if(TextPath.IsEmpty())
		{
			return LOCTEXT("EmptyBinding", "Access is not bound to anything");
		}
		else
		{
			IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
			const FText UnderlyingPath = PropertyAccessEditor.MakeTextPath(K2Node_PropertyAccess->GetPath());	
			const FText& CompilationContext = K2Node_PropertyAccess->GetCompiledContext();
			const FText& CompilationContextDesc = K2Node_PropertyAccess->GetCompiledContextDesc();
			if(CompilationContext.IsEmpty() && CompilationContextDesc.IsEmpty())
			{

				return FText::Format(LOCTEXT("ToolTipFormat", "Access property '{0}'\nNative: {1}"), TextPath, UnderlyingPath);
			}
			else
			{
				return FText::Format(LOCTEXT("CompiledAccessToolTipFormat", "Access property '{0}'\nNative: {1}\n{2}\n{3}"), TextPath, UnderlyingPath, CompilationContext, CompilationContextDesc);
			}
		}
	});
	
	Args.CurrentBindingImage = MakeAttributeLambda([K2Node_PropertyAccess]()
	{
		if(const FProperty* Property = K2Node_PropertyAccess->GetResolvedProperty())
		{
			if(Cast<UFunction>(Property->GetOwnerUField()) != nullptr)
			{
				static FName FunctionIcon(TEXT("GraphEditor.Function_16x"));
				return FAppStyle::GetBrush(FunctionIcon);
			}
			else
			{
				const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

				FEdGraphPinType PinType;
				Schema->ConvertPropertyToPinType(K2Node_PropertyAccess->GetResolvedProperty(), PinType);
				return FBlueprintEditorUtils::GetIconFromPin(PinType, true);
			}
		}

		static FName PropertyIcon(TEXT("Kismet.Tabs.Variables"));
		return FAppStyle::GetBrush(PropertyIcon);
	});

	Args.CurrentBindingColor = MakeAttributeLambda([K2Node_PropertyAccess]()
	{
		if(K2Node_PropertyAccess->GetResolvedProperty())
		{
			const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

			FEdGraphPinType PinType;
			Schema->ConvertPropertyToPinType(K2Node_PropertyAccess->GetResolvedProperty(), PinType);

			return Schema->GetPinTypeColor(PinType);
		}
		else
		{
			return FLinearColor(0.5f, 0.5f, 0.5f);
		}
	});

	IPropertyAccessBlueprintBinding::FContext BindingContext;
	BindingContext.Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(K2Node_PropertyAccess);
	BindingContext.Graph = K2Node_PropertyAccess->GetGraph();
	BindingContext.Node = K2Node_PropertyAccess;
	BindingContext.Pin = K2Node_PropertyAccess->FindPin(TEXT("Value"));

	IPropertyAccessBlueprintBinding::FBindingMenuArgs MenuArgs;
	MenuArgs.OnSetPropertyAccessContextId = FOnSetPropertyAccessContextId::CreateLambda([K2Node_PropertyAccess](const FName& InContextId)
	{
		K2Node_PropertyAccess->Modify();
		K2Node_PropertyAccess->SetContextId(InContextId);
		FBlueprintEditorUtils::MarkBlueprintAsModified(K2Node_PropertyAccess->GetBlueprint());
	});
	MenuArgs.OnCanSetPropertyAccessContextId = FOnCanSetPropertyAccessContextId::CreateLambda([K2Node_PropertyAccess](const FName& InContextId)
	{
		return K2Node_PropertyAccess->HasResolvedPinType();
	});	
	MenuArgs.OnGetPropertyAccessContextId = FOnGetPropertyAccessContextId::CreateLambda([K2Node_PropertyAccess]()
	{
		return K2Node_PropertyAccess->GetContextId();
	});
	
	// Check any property access blueprint bindings we might have registered for menu extenders
	TArray<TSharedPtr<FExtender>> Extenders;
	for(IPropertyAccessBlueprintBinding* Binding : IModularFeatures::Get().GetModularFeatureImplementations<IPropertyAccessBlueprintBinding>("PropertyAccessBlueprintBinding"))
	{
		TSharedPtr<FExtender> BindingExtender = Binding->MakeBindingMenuExtender(BindingContext, MenuArgs);
		if(BindingExtender)
		{
			Extenders.Add(BindingExtender);
		}
	}
	
	if(Extenders.Num() > 0)
	{
		Args.MenuExtender = FExtender::Combine(Extenders);
	}

	Args.bAllowArrayElementBindings = true;
	Args.bAllowNewBindings = false;
	Args.bAllowUObjectFunctions = true;
	Args.bAllowStructFunctions = true;

	TSharedPtr<SWidget> PropertyBindingWidget;

	if(IModularFeatures::Get().IsModularFeatureAvailable("PropertyAccessEditor"))
	{
		IPropertyAccessEditor& PropertyAccessEditor = IModularFeatures::Get().GetModularFeature<IPropertyAccessEditor>("PropertyAccessEditor");
		PropertyBindingWidget = PropertyAccessEditor.MakePropertyBindingWidget(K2Node_PropertyAccess->GetBlueprint(), Args);
	}
	else
	{
		PropertyBindingWidget = SNullWidget::NullWidget;
	}
	
	const FTextBlockStyle& TextBlockStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PropertyAccess.CompiledContext.Text");
	
	InTitleWidget =
		SNew(SLevelOfDetailBranchNode)
		.UseLowDetailSlot(this, &SPropertyAccessNode::UseLowDetailNodeTitles)
		.LowDetail()
		[
			SNew(SSpacer)
		]
		.HighDetail()
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				PropertyBindingWidget.ToSharedRef()
			]
			+SOverlay::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			[
				SNew(SBorder)
				.Padding(1.0f)
				.BorderImage(FAppStyle::GetBrush("PropertyAccess.CompiledContext.Border"))
				.RenderTransform_Lambda([K2Node_PropertyAccess, &TextBlockStyle]()
				{
					const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
					FVector2D TextSize = FontMeasureService->Measure(K2Node_PropertyAccess->GetCompiledContext(), TextBlockStyle.Font);
					return FSlateRenderTransform(FVector2D(0.0f, TextSize.Y));
				})	
				[
					SNew(STextBlock)
					.TextStyle(&TextBlockStyle)
					.Visibility_Lambda([K2Node_PropertyAccess]()
					{
						return K2Node_PropertyAccess->GetCompiledContext().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
					})
					.Text_Lambda([K2Node_PropertyAccess]()
					{
						return K2Node_PropertyAccess->GetCompiledContext();
					})
				]
			]
		];

	InTitleWidget->SetCursor( EMouseCursor::Default );

	InOutTitleHAlign = HAlign_Left;
	InOutTitleMargin = FMargin(40.0f, 14.0f, 40.0f, 14.0f);
		
	return InTitleWidget.ToSharedRef();
}


TSharedPtr<SGraphPin> SPropertyAccessNode::CreatePinWidget(UEdGraphPin* Pin) const
{
	TSharedPtr<SGraphPin> DefaultWidget = SGraphNodeK2Var::CreatePinWidget(Pin);
	DefaultWidget->SetShowLabel(false);

	return DefaultWidget;
}

#undef LOCTEXT_NAMESPACE
