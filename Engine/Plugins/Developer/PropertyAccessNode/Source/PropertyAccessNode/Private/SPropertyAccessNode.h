// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetNodes/SGraphNodeK2Var.h"

class UK2Node_PropertyAccess;

class SPropertyAccessNode : public SGraphNodeK2Var
{
public:
	SLATE_BEGIN_ARGS(SPropertyAccessNode) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, UK2Node_PropertyAccess* InNode);

	// SGraphNode interface
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	virtual TSharedRef<SWidget> UpdateTitleWidget(FText InTitleText, TSharedPtr<SWidget> InTitleWidget, EHorizontalAlignment& InOutTitleHAlign, FMargin& InOutTitleMargin) const override;

private:
	// Helper for property/function binding
	bool CanBindProperty(FProperty* InProperty) const;
};
