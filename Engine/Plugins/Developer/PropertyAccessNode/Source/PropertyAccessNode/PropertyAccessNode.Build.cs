// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PropertyAccessNode : ModuleRules
{
	public PropertyAccessNode(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Slate",	
				"GraphEditor",
				"BlueprintGraph",
				"Engine",
				"UnrealEd",
				"SlateCore",
				"InputCore",
				"KismetWidgets",
				"KismetCompiler",
				"PropertyAccessEditor",
				"AnimGraph",
			}
		);
	}
}
